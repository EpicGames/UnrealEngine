{
	"AvdDevice": "pixel_6",
	"DeviceApiVersion": "35",
	"DeviceDataSize": "48G",
	"DeviceRamSize": "6144"
}
