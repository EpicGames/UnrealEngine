[/Script/Engine.Engine]
; DefaultBloomKernel is not used on mobile
DefaultBloomKernelTextureName=/Engine/EngineResources/DefaultTexture.DefaultTexture

; Default film grain is not used on mobile
DefaultFilmGrainTextureName=""

[PlatformCrypto]
PlatformRequiresDataCrypto=True
PakSigningRequired=False

[Audio]
PlatformFormat=OGG
PlatformStreamingFormat=OGG

; Defining below allows switching to audio mixer using -audiomixer commandline
AudioMixerModuleName=AudioMixerAndroid

; Defines a platform-specific volume headroom (in dB) for audio to provide better platform consistency with respect to volume levels.
PlatformHeadroomDB=-6

[PlatformMemoryBuckets]
LargestMemoryBucket_MinGB=8
LargerMemoryBucket_MinGB=6
DefaultMemoryBucket_MinGB=4
SmallerMemoryBucket_MinGB=3
 ; for now, we require 3gb
SmallestMemoryBucket_MinGB=3


[DeviceProfileManager]
DeviceProfileSelectionModule="AndroidDeviceProfileSelectorRuntime"
PreviewDeviceProfileSelectionModule="AndroidDeviceProfileSelector"

[Advertising]
DefaultProviderName=AndroidAdvertising

[/Script/TcpMessaging.TcpMessagingSettings]
EnableTransport=True
ListenEndpoint=127.0.0.1:6666
!ConnectToEndpoints=CLEAR_ARRAY
ConnectionRetryDelay=0

[LocalNotification]
DefaultPlatformService=AndroidLocalNotification

[DeviceScreenDensity]
+Devices=(Model="Nexus 5",Density=445)
+Devices=(Model="Nexus 5X",Density=424)
+Devices=(Model="Nexus 6",Density=493)
+Devices=(Model="Nexus 6P",Density=518)
+Devices=(Model="SM-G930.+",Density=577,IsRegex=true) ; Samsung Galaxy S7
+Devices=(Model="SM-G935.+",Density=534,IsRegex=true) ; Samsung Galaxy S7 Edge
+Devices=(Model="SM-N920.+",Density=515,IsRegex=true) ; Samsung Galaxy Note 5
+Devices=(Model="Pixel",Density=441)
+Devices=(Model="Pixel C",Density=308)
+Devices=(Model="Pixel XL",Density=534)
+Devices=(Model="HTC 10",Density=564)
+Devices=(Model="EVA-L09",Density=423) ; Huawei P9
+Devices=(Model="A063", Density=420) ; Nothing Phone


;GalaxyS5=432
;GalaxyS6=577

[ConsoleVariables]
Slate.CacheRenderData=0
r.VSync=1
; high priority thread tasks can be downgraded to normal priority if a task is created with EPowerSavingEligibility::Eligible.
TaskGraph.EnablePowerSavingThreadPriorityReduction=1
; prefer D24_S8 for depth/stencil
r.Vulkan.Depth24Bit=1
; Support Mali 64k texel buffer limitation, tuned for 1080p. If you target 2k-4k resolutions please also set the LightGridPixelSize to 128
r.Forward.LightGridSizeZ = 8
r.Forward.MaxCulledLightsPerCell = 8
r.HairStrands.Simulation=0
r.HairStrands.Strands=0
r.HairStrands.Binding=0
; Do not backtrace vulkan memory allocations
r.Vulkan.MemoryBacktrace=0
; Disable Vulkan Submission AllowTimelineSemaphores as it seems to cause an FD leak on adreno drivers.
r.Vulkan.Submission.AllowTimelineSemaphores=0
; Disable RHI multi-threading until its effects are measured on older devices
r.Vulkan.Submission.UseInterruptThread=0
r.Vulkan.Submission.UseSubmissionThread=0
r.Vulkan.RHIThread=1
r.Vulkan.UseMinimalSubmits=1
; Disable async compute until we can target devices to enable it on
r.Vulkan.AllowAsyncCompute=0
; Disable split barriers until its effects are measured on all devices
r.Vulkan.AllowSplitBarriers=0
; Disable dynamic rendering until its effects are measured on all devices
r.Vulkan.AllowDynamicRendering=0
; Crash reporting diagnotic buffer
r.Vulkan.DiagnosticBuffer=0
; Transient memory allocator
r.Vulkan.EnableTransientResourceAllocator=0
; Disable sparse binding and residency
r.Vulkan.AllowSparseResidency=0
; Use renames instead of copy+barrier on mobile devices
r.Vulkan.AllowUniformUpload=0
; Experiment to try specialization constants on a single shader (disabled by default)
r.LightGridInjection.UseSpecializationConstants=0

; enable stripping debug strings from streamable manager in shipping to save memory
s.StreamableStripDebugName=true
s.StreamableStripDebugNameInShipping=true

[AlternateTextureCompression]
;disabling TextureCompressionFormat for Android/IOS no longer needed
;TextureCompressionFormat=""
;TextureFormatPrefix=""

[SystemSettings]
;Perplatform to PerQualityLevel conversion mapping for platform groups
QualityLevelMapping="low"

;Manual range settings for Niagara quality levels. Simplifies handling of device profiles/fragments and what quality levels they can set.
;Please ensure device profiles or fragments for this platform do not set fx.Niagara.QualityLevel values outside of this range
fx.Niagara.QualityLevel.Min=0
fx.Niagara.QualityLevel.Max=1
