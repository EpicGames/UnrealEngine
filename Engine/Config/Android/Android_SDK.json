{
	"MainVersion": "r27c",
	"MinVersion": "r27c",
	"MaxVersion": "r29",

	"AutoSDKDirectory": "-28",

	"platforms": "android-34",
	"build-tools": "35.0.1",
	"cmake": "3.22.1",
	"ndk": "27.2.12479018"
}
