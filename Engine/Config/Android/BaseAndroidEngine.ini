[OnlineSubsystem]
DefaultPlatformService=GooglePlay
NativePlatformService=GooglePlay
LocalPlatformName=AND

[OnlineSubsystemGooglePlay.Store]
bSupportsInAppPurchasing=true

[Audio]
AudioCaptureModuleName=AudioCaptureAndroid

;Uncomment to turn on AndroidBackgroundHttp through the AndroidFetchBackgroundDownload and AndroidBackgroundService modules
;[BackgroundHttp]
;PlatformModularFeatureName=AndroidFetchBackgroundDownload

[BuildPatchTool]
VerificationThreadCount=3

[/Script/Engine.GarbageCollectionSettings]
gc.MaxObjectsInGame=131072

[SlateRenderer]
NumPreallocatedVertices=200

[/Script/Engine.RendererSettings]
r.DistanceFields=0
r.TemporalAA.Mobile.UseCompute=0
; Disables parallel GDME by default due to not enough threads
r.Visibility.DynamicMeshElements.Parallel=0
; Volumetric Clouds are not supported by mobile renderer, skip materials/global shaders compilation.
r.VolumetricCloud.Support=0
r.OIT.SortedTriangles=0

[/Script/AndroidRuntimeSettings.AndroidRuntimeSettings]
; Default to latest astc version - projects can manually specify this to retain older versions to prevent
; patches.
ASTCVersion=501

[MemoryMappedFiles]
Enable=true
Alignment=1
