[InstallBundleSource.Bulk.MiscSettings]
; Allows our Bulk data source to serialize a BulkBuildData.ini if it was missing during loading.
; Can drastically speed up Bulk data source initialization for runs after the first as it won't have to compute
; Bulk bundle information during init.
bShouldSerializeMissingBulkBuildDataIni=true
; If we limit our file parsing in Bulk bundle sources to just .pak files
bOnlyGatherPaksInBulkData=true