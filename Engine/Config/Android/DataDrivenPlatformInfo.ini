[DataDrivenPlatformInfo]
bIsConfidential=false
TargetSettingsIniSectionName=/Script/AndroidRuntimeSettings.AndroidRuntimeSettings
bHasDedicatedGamepad=false
bInputSupportConfigurable=false
DefaultInputType=Touch
bSupportsMouseAndKeyboard=false
bSupportsGamepad=true
bCanChangeGamepadType=true
bSupportsTouch=true
GlobalIdentifier=3619EA87DE704A48BB1551754423C26A

NormalIconPath=Launcher/Android/Platform_Android_24x
LargeIconPath=Launcher/Android/Platform_Android_128x
XLargeIconPath=
AutoSDKPath=Android
TutorialPath=SharingAndReleasing/Mobile/Android
bIsEnabled=true
bUsesHostCompiler=false
bUATClosesAfterLaunch=false
PlatformGroupName=Mobile
ICUDataVersion=icudt64l

[PreviewPlatform AndroidES31]
EnabledCVar=ini:Engine:Android:/Script/AndroidRuntimeSettings.AndroidRuntimeSettings:bBuildForES31
PlatformName=Android
ShaderPlatform=OPENGL_ES3_1_ANDROID
ShaderFormat=GLSL_ES3_1_ANDROID
ActiveIconName=LevelEditor.PreviewMode.AndroidES31.Enabled
InactiveIconName=LevelEditor.PreviewMode.AndroidES31.Disabled
MenuTooltip=NSLOCTEXT("PreviewPlatform", "PreviewMenuTooltip_AndroidOpenGL ", "Mobile preview using Android OpenGL quality settings.")
IconText=NSLOCTEXT("PreviewPlatform", "PreviewIconText_AndroidOpenGL", "Android OpenGL")
DeviceProfileName=Android_Preview_OpenGL:Android_Low:Android_Mid:Android_High
FriendlyName=NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_GenericOpenglAndroid", "Generic Android OpenGL"):NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_AndroidOpenGLLow","Android OpenGL Low"):NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_AndroidOpenGLMid","Android OpenGL Mid"):NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_AndroidOpenGLHigh","Android OpenGL High")


[PreviewPlatform AndroidVulkan]
EnabledCVar=ini:Engine:Android:/Script/AndroidRuntimeSettings.AndroidRuntimeSettings:bSupportsVulkan
PlatformName=Android
ShaderPlatform=VULKAN_ES3_1_ANDROID
ShaderFormat=SF_VULKAN_ES31_ANDROID
ActiveIconName=LevelEditor.PreviewMode.AndroidVulkan.Enabled
InactiveIconName=LevelEditor.PreviewMode.AndroidVulkan.Disabled
MenuTooltip=NSLOCTEXT("PreviewPlatform", "PreviewMenuTooltip_AndroidVulkan", "Mobile preview using Android Vulkan quality settings.")
IconText=NSLOCTEXT("PreviewPlatform", "PreviewIconText_AndroidVulkan", "Android Vulkan")
DeviceProfileName=Android_Preview_Vulkan:Android_Low:Android_Mid:Android_High:Android_OpenXR
FriendlyName=NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_GenericVulkanAndroid", "Generic Android Vulkan"):NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_AndroidVulkanLow","Android Vulkan Low"):NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_AndroidVulkanMid","Android Vulkan Mid"):NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_AndroidVulkanHigh","Android Vulkan High"):NSLOCTEXT("PreviewPlatform", "PreviewFriendlyName_AndroidGenericXR","Android Generic XR")


[PreviewPlatform AndroidVulkanSM5]
EnabledCVar=ini:Engine:Android:/Script/AndroidRuntimeSettings.AndroidRuntimeSettings:bSupportsVulkanSM5
PlatformName=Android
ShaderPlatform=VULKAN_SM5_ANDROID
ShaderFormat=SF_VULKAN_SM5_ANDROID
PreviewFeatureLevel=SM6
ActiveIconName=LevelEditor.PreviewMode.AndroidVulkanSM5.Enabled
InactiveIconName=LevelEditor.PreviewMode.AndroidVulkanSM5.Disabled
MenuTooltip=NSLOCTEXT("PreviewPlatform", "PreviewMenuTooltip_AndroidVulkanSM5", "Mobile preview using Android Vulkan SM5 quality settings.")
IconText=NSLOCTEXT("PreviewPlatform", "PreviewIconText_AndroidVulkanSM5", "Android VK SM5")
DeviceProfileName=Android_Preview_Vulkan_SM5

[ShaderPlatform OPENGL_ES3_1_ANDROID]
Language=OpenGL
MaxFeatureLevel=ES3_1
ShaderFormat=GLSL_ES3_1_ANDROID
bIsMobile=true
bIsAndroidOpenGLES=true
bTargetsTiledGPU=true
bSupportsMobileMultiView=false
bNeedsOfflineCompiler=true
bSupportsUnrestrictedHalfFloatBuffers=false
bSupportsManualVertexFetch=false
bSupportsClipDistance = false
bSupportsShaderPipelines = false
bSupportsVertexShaderSRVs=false
bSupportsUniformBufferObjects = true
bSupportsDxc=true
bSupportsIndependentSamplers=false
bSupportsComputeFramework = true
bSupportsDistanceFields=false
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_Android", "Android OpenGL Mobile")

[ShaderPlatform VULKAN_ES3_1_ANDROID]
Language=Vulkan
MaxFeatureLevel=ES3_1
ShaderFormat=SF_VULKAN_ES31_ANDROID
bIsMobile=true
bSupportsSpecializationConstants=true
bTargetsTiledGPU=true
bSupportsMobileMultiView=true
; The support really is conditional, the runtime code needs to also test GRHISupportsArrayIndexFromAnyShader
bSupportsVertexShaderLayer=true
bNeedsOfflineCompiler=true
bSupportsIndexBufferUAVs=true
bSupportsManualVertexFetch=false
bSupportsClipDistance = false
bSupportsShaderPipelines = false
bSupportsVariableRateShading = true
bSupportsVertexShaderSRVs=false
bSupportsUniformBufferObjects = true
bSupportsDualSourceBlending = false
bSupportsDxc=true
bSupportsIndependentSamplers=true
bSupportsComputeFramework = true
bSupportsDistanceFields=true

FriendlyName=LOCTEXT("FriendlyShaderPlatformName_AndroidVulkan", "Android Vulkan Mobile")

[ShaderPlatform VULKAN_SM5_ANDROID]
Language=Vulkan
MaxFeatureLevel=SM5
ShaderFormat=SF_VULKAN_SM5_ANDROID
bTargetsTiledGPU=true
bNeedsOfflineCompiler=true
bSupportsIndexBufferUAVs=true
bRequiresDisableForwardLocalLights=true
bWaterUsesSimpleForwardShading=true
bSupportsVertexShaderSRVs=true
bSupportsManualVertexFetch=true
bSupportsUniformBufferObjects = true
; Raytracing and Lumen
bSupportsRayTracing = true
bSupportsRayTracingShaders = false
bSupportsInlineRayTracing = true
bSupportsRayTracingIndirectInstanceData = true
bEnableRayTracing=true
bSupportsGPUSkinCache=true
EnablesHLSL2021ByDefault=1
bSupportsGPUScene=true
bSupportsLumenGI=true
bSupportsDistanceFields=true
bSupportsDualSourceBlending=false
bSupportsVariableRateShading = false
bSupportsDxc=true
bSupportsIndependentSamplers=true
bSupportsWaveOperations=RuntimeDependent
MinimumWaveSize=4
MaximumWaveSize=128
bSupportsWave64=true
bSupportsComputeFramework = true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_AndroidVulkanSM5", "Android Vulkan SM5 Mobile")

